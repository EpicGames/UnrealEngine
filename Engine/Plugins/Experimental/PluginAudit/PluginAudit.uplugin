{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Plugin Audit",
	"Description" : "Editor plugin for auditing plugin connectivity.",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "http://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	
	"Modules" :
	[
		{
			"Name": "PluginAudit",
			"Type": "Editor",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "GameFeatures",
			"Enabled": true
		},
		{
			"Name": "AssetManagerEditor",
			"Enabled": true
		},
		{
			"Name": "PluginReferenceViewer",
			"Enabled": true
		}
	]
}