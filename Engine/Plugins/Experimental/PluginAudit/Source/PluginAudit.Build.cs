// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PluginAudit : ModuleRules
{
	public PluginAudit(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"StatsViewer",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
				"CoreUObject",
				"Engine",
				"TargetPlatform",
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Slate",
				"SlateCore",
                "ApplicationCore",
                "InputCore",
				"EditorFramework",
				"UnrealEd",
				"AssetRegistry",
				"Json",
				"CollectionManager",
				"ContentBrowser",
				"ContentBrowserData",
				"WorkspaceMenuStructure",
				"AssetDefinition",
				"AssetTools",
				"PropertyEditor",
				"GraphEditor",
				"BlueprintGraph",
				"KismetCompiler",
				"MessageLog",
				"LevelEditor",
				"SandboxFile",
				"EditorWidgets",
				"TreeMap",
				"ToolMenus",
				"ToolWidgets",
				"SourceControl",
				"SourceControlWindows",
				"UncontrolledChangelists",
				"Projects",
				"GameFeatures",
				"GameplayTags",
				"AssetManagerEditor",
				"PluginReferenceViewer",
			}
		);
	}
}
