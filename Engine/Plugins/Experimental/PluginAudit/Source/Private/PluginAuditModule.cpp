// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "SPluginAuditBrowser.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "PluginAudit"

class FPluginAuditModule : public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End of IModuleInterface interface

private:
	static const FName PluginAuditTabName;

	TSharedRef<SDockTab> SpawnPluginAuditTab(const FSpawnTabArgs& Args);
};

const FName FPluginAuditModule::PluginAuditTabName = TEXT("PluginAudit");

///////////////////////////////////////////

IMPLEMENT_MODULE(FPluginAuditModule, PluginAudit);


void FPluginAuditModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(PluginAuditTabName, FOnSpawnTab::CreateRaw(this, &FPluginAuditModule::SpawnPluginAuditTab))
		.SetDisplayName(LOCTEXT("PluginAuditTitle", "Plugin Audit"))
		.SetTooltipText(LOCTEXT("PluginAuditTooltip", "Open Plugin Audit window, allows viewing detailed information about plugin references."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsAuditCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Audit"));
	FGlobalTabmanager::Get()->RegisterDefaultTabWindowSize(PluginAuditTabName, FVector2D(1080, 600));
}

void FPluginAuditModule::ShutdownModule()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(PluginAuditTabName);
}

TSharedRef<SDockTab> FPluginAuditModule::SpawnPluginAuditTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SPluginAuditBrowser)
		];
}

#undef LOCTEXT_NAMESPACE
