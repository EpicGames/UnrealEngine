// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPluginAuditBrowser.h"

#include "Features/EditorFeatures.h"
#include "Features/IModularFeatures.h"
#include "Features/IPluginsEditorFeature.h"
#include "Filters/SFilterSearchBox.h"
#include "GameFeaturesProjectPolicies.h"
#include "GameFeaturesSubsystem.h"
#include "GameplayTagsManager.h"
#include "GameplayTagsSettings.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"
#include "Interfaces/IPluginManager.h"
#include "Algo/Copy.h"
#include "Misc/Paths.h"
#include "Async/ParallelFor.h"
#include "Misc/ScopedSlowTask.h"
#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#include "Misc/UObjectToken.h"
#include "PluginReferenceViewerModule.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Input/SCheckBox.h"
#include "AssetManagerEditorModule.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "SPluginAuditBrowser"

namespace PluginAudit
{
	FName PluginAuditLogName = TEXT("Plugin Audit");
}

void SPluginAuditBrowser::Construct(const FArguments& InArgs)
{
	BuildPluginList();
	CreateLogListing();

	// Setup text filtering
	PluginTextFilter = MakeShareable(new FCookedPluginTextFilter(FCookedPluginTextFilter::FItemToStringArray::CreateLambda([](const IPlugin* Plugin, TArray<FString>& OutStringArray)
		{
			OutStringArray.Add(Plugin->GetFriendlyName());
		})
	));

	PluginTextFilter->OnChanged().AddSP(this, &SPluginAuditBrowser::OnPluginTextFilterChanged);

	RebuildAndFilterPluginList();

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	RefreshToolBar();

	PluginListView =
		SNew(SListView< TSharedRef<FCookedPlugin> >)
		.ListItemsSource(&FilteredCookedPlugins)
		.OnGenerateRow(this, &SPluginAuditBrowser::MakeCookedPluginRow)
		.OnContextMenuOpening(this, &SPluginAuditBrowser::OnContextMenuOpening)
		.OnMouseButtonDoubleClick(this, &SPluginAuditBrowser::OnListViewDoubleClick);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
			[
				UToolMenus::Get()->GenerateWidget("PluginAudit.MainToolBar", FToolMenuContext())
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)

			+ SSplitter::Slot()
			.Value(0.30f)
			[
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.f, 4.0f, 0.0f, 6.0f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()				
						[
							SNew(SCheckBox)
							.ToolTipText(LOCTEXT("ToggleAll", "Toggle disabled for all visible plugins"))
							.IsChecked(this, &SPluginAuditBrowser::GetGlobalDisabledState)
							.OnCheckStateChanged(this, &SPluginAuditBrowser::OnGlobalDisabledStateChanged)
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							SAssignNew(SearchBoxPtr, SSearchBox)
							.OnTextChanged(this, &SPluginAuditBrowser::SearchBox_OnPluginSearchTextChanged)
						]
					]

					+ SVerticalBox::Slot()
					[
						PluginListView.ToSharedRef()
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)

							// Asset count
						+ SHorizontalBox::Slot()
						.FillWidth(1.f)
						.VAlign(VAlign_Center)
						.Padding(8, 5)
						[
							SNew(STextBlock)
							.Text(this, &SPluginAuditBrowser::GetPluginCountText)
						]
					]
				]
			]
			+ SSplitter::Slot()
			.Value(0.90f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					MessageLogModule.CreateLogListingWidget(LogListing.ToSharedRef())
				]
			]
		]
	];
}

void SPluginAuditBrowser::BuildPluginList()
{
	const UGameFeaturesSubsystem& GameFeaturesSubsystem = UGameFeaturesSubsystem::Get();
	const UGameFeaturesProjectPolicies& Policy = UGameFeaturesSubsystem::Get().GetPolicy();

	TArray<TSharedRef<IPlugin>> AllPlugins = IPluginManager::Get().GetDiscoveredPlugins();

	IncludedGameFeaturePlugins.Reset();
	ExcludedGameFeaturePlugins.Reset();
	for (const TSharedRef<IPlugin>& Plugin : AllPlugins)
	{
		FGameFeaturePluginDetails PluginDetails;
		if (GameFeaturesSubsystem.GetBuiltInGameFeaturePluginDetails(Plugin, PluginDetails))
		{
			if (Policy.WillPluginBeCooked(Plugin->GetDescriptorFileName(), PluginDetails))
			{
				// We're actually including this plugin.
				IncludedGameFeaturePlugins.Add(Plugin);
				continue;
			}

			ExcludedGameFeaturePlugins.Add(Plugin);
		}
	}

	for (const TSharedRef<IPlugin>& Plugin : IncludedGameFeaturePlugins)
	{
		CookedPlugins.Add(MakeShared<FCookedPlugin>(Plugin));
	}
}

void SPluginAuditBrowser::RebuildAndFilterPluginList()
{
	FilteredCookedPlugins.Empty();

	for (const auto& CookedPlugin : CookedPlugins)
	{
		if (PluginTextFilter->PassesFilter(&CookedPlugin->Plugin.Get()))
		{
			FilteredCookedPlugins.Add(CookedPlugin);
		}
	}

	FilteredCookedPlugins.Sort([](const TSharedRef<FCookedPlugin>& A, const TSharedRef<FCookedPlugin>& B)
	{
		return A->Plugin->GetName() < B->Plugin->GetName();
	});
}

void SPluginAuditBrowser::RefreshToolBar()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* MainToolBar = UToolMenus::Get()->RegisterMenu("PluginAudit.MainToolBar", NAME_None, EMultiBoxType::SlimHorizontalToolBar);
	MainToolBar->StyleName = "AssetEditorToolbar";
	{
		FToolMenuSection& PlaySection = MainToolBar->AddSection("Actions");

		FToolMenuEntry SaveEntry = FToolMenuEntry::InitToolBarButton(
			"Refresh",
			FUIAction(FExecuteAction::CreateSP(this, &SPluginAuditBrowser::RefreshViolations)),
			LOCTEXT("SaveDirtyPackages", "Refresh"),
			LOCTEXT("SaveDirtyPackagesTooltip", "Refreshes the audit results based on the enabled plugins."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Refresh")
		);

		PlaySection.AddEntry(SaveEntry);
	}
}

void SPluginAuditBrowser::OnPluginTextFilterChanged()
{
	RebuildAndFilterPluginList();

	PluginListView->RequestListRefresh();
}

void SPluginAuditBrowser::SearchBox_OnPluginSearchTextChanged(const FText& NewText)
{
	PluginTextFilter->SetRawFilterText(NewText);
	SearchBoxPtr->SetError(PluginTextFilter->GetFilterErrorText());
}

void SPluginAuditBrowser::OnGlobalDisabledStateChanged(ECheckBoxState State)
{
	bGlobalDisabledState = (State == ECheckBoxState::Checked) ? false : true;
	
	for (auto& Item : FilteredCookedPlugins)
	{
		Item->bSimulateDisabled = bGlobalDisabledState;
	}
}

ECheckBoxState SPluginAuditBrowser::GetGlobalDisabledState() const
{
	bool bStateMismatch = false;
	for (const auto& Item : FilteredCookedPlugins)
	{
		if (Item->bSimulateDisabled != bGlobalDisabledState)
		{
			bStateMismatch = true;
			break;
		}
	}

	return bStateMismatch ? ECheckBoxState::Undetermined :
		bGlobalDisabledState ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}

FText SPluginAuditBrowser::GetPluginCountText() const
{
	const int32 NumPlugins = CookedPlugins.Num();
	const int32 NumPluginsDisabled = CookedPlugins.FilterByPredicate([](const auto& Item) { return Item->bSimulateDisabled; }).Num();

	FText PluginCount = FText::GetEmpty();
	if (NumPlugins == 1)
	{
		PluginCount = FText::Format(LOCTEXT("PluginCountLabelSingularPlusDisabled", "1 plugin ({0} disabled)"), FText::AsNumber(NumPluginsDisabled));
	}
	else
	{
		PluginCount = FText::Format(LOCTEXT("PluginCountLabelPluralPlusDisabled", "{0} plugins ({1} disabled)"), FText::AsNumber(CookedPlugins.Num()), FText::AsNumber(NumPluginsDisabled));
	}

	return PluginCount;
}

TSharedRef<ITableRow> SPluginAuditBrowser::MakeCookedPluginRow(TSharedRef<SPluginAuditBrowser::FCookedPlugin> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow< TSharedRef<FCookedPlugin> >, OwnerTable)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([InItem]() -> ECheckBoxState
			{
				return InItem->bSimulateDisabled ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
			})
			.OnCheckStateChanged_Lambda([InItem](ECheckBoxState State)
			{
				InItem->bSimulateDisabled = (State != ECheckBoxState::Checked);
			})
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::FromString(InItem->Plugin->GetFriendlyName()))
		]
	];
}

TSharedPtr<SWidget> SPluginAuditBrowser::OnContextMenuOpening()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("OpenPluginPropertiesLabel", "Open Plugin Properties"),
		LOCTEXT("OpenPluginPropertiesTooltip", "Open Plugin Properties Window"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPluginAuditBrowser::OnOpenPluginProperties),
			FCanExecuteAction()
		));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("OpenPluginReferenceViewerLabel", "Open Plugin Reference Viewer"),
		LOCTEXT("OpenPluginReferenceViewerTooltip", "Open Plugin Reference Viewer"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPluginAuditBrowser::OnOpenPluginReferenceViewer),
			FCanExecuteAction()
		));

	return MenuBuilder.MakeWidget();
}

void SPluginAuditBrowser::OnListViewDoubleClick(TSharedRef<FCookedPlugin> Item)
{
	OpenPluginReferenceViewer(Item->Plugin);
}

void SPluginAuditBrowser::OnOpenPluginProperties()
{
	TArray<TSharedRef<FCookedPlugin>> SelectedItems = PluginListView->GetSelectedItems();
	if (SelectedItems.Num() == 1)
	{
		OpenPluginProperties(SelectedItems[0]->Plugin->GetName());
	}
}

void SPluginAuditBrowser::OnOpenPluginReferenceViewer()
{
	TArray<TSharedRef<FCookedPlugin>> SelectedItems = PluginListView->GetSelectedItems();
	if (SelectedItems.Num() == 1)
	{
		OpenPluginReferenceViewer(SelectedItems[0]->Plugin);
	}
}

void SPluginAuditBrowser::OpenPluginProperties(const FString& PluginName)
{
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
	if (Plugin != nullptr)
	{
		IPluginsEditorFeature& PluginEditor = IModularFeatures::Get().GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor);
		PluginEditor.OpenPluginEditor(Plugin.ToSharedRef(), nullptr, FSimpleDelegate());
	}
}

void SPluginAuditBrowser::OpenPluginReferenceViewer(const TSharedRef<IPlugin>& Plugin)
{
	FPluginReferenceViewerModule::Get().OpenPluginReferenceViewerUI(Plugin);
}

void SPluginAuditBrowser::CreateLogListing()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	FMessageLogInitializationOptions LogOptions;
	LogOptions.bShowInLogWindow = false;
	LogOptions.bAllowClear = false;
	LogOptions.bShowPages = false;
	LogOptions.bShowFilters = false;
	LogOptions.MaxPageCount = 1;

	LogListing = MessageLogModule.CreateLogListing(PluginAudit::PluginAuditLogName, LogOptions);
}

void SPluginAuditBrowser::RefreshViolations()
{
	TArray<TSharedRef<IPlugin>> IncludedPlugins = IncludedGameFeaturePlugins;
	TArray<TSharedRef<IPlugin>> ExcludedPlugins = ExcludedGameFeaturePlugins;
	for (const TSharedRef<FCookedPlugin>& CookedPlugin : CookedPlugins)
	{
		if (CookedPlugin->bSimulateDisabled)
		{
			IncludedPlugins.Remove(CookedPlugin->Plugin);
			ExcludedPlugins.Add(CookedPlugin->Plugin);
		}
	}

	TArray<TSharedRef<FTokenizedMessage>> Violations = ScanForViolations(IncludedPlugins, ExcludedPlugins);

	LogListing->ClearMessages();

	for (const TSharedRef<FTokenizedMessage>& Violation : Violations)
	{
		LogListing->AddMessage(Violation);
	}
}

/*static*/ TArray<TSharedRef<FTokenizedMessage>> SPluginAuditBrowser::ScanForViolations(TArray<TSharedRef<IPlugin>> InIncludedGameFeaturePlugins, TArray<TSharedRef<IPlugin>> InExcludedGameFeaturePlugins)
{
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	AssetRegistry->WaitForCompletion();
	
    UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	TArray<FGameFeaturePlugin> ExcludedPlugins;
	GetGameFeaturePlugins(InExcludedGameFeaturePlugins, ExcludedPlugins);

	TArray<FGameFeaturePlugin> IncludedPlugins;
	GetGameFeaturePlugins(InIncludedGameFeaturePlugins, IncludedPlugins);

	TArray<TSharedRef<FTokenizedMessage>> Violations;
	FCriticalSection CS;

	FScopedSlowTask SlowTask(IncludedPlugins.Num() + ExcludedPlugins.Num(), LOCTEXT("Examining Plugins", "Examining Plugins..."));
	SlowTask.MakeDialog(true);

	const FName GameplayTagStructPackage = FGameplayTag::StaticStruct()->GetOutermost()->GetFName();
	const FName NAME_GameplayTag = FGameplayTag::StaticStruct()->GetFName();

	bool bCancelled = false;
	
	// Included plugins.
	for (const FGameFeaturePlugin& Plugin : IncludedPlugins)
	{
		if (SlowTask.ShouldCancel())
		{
			bCancelled = true;
			break;
		}

		SlowTask.EnterProgressFrame(1.f, FText::FromString(Plugin.Plugin->GetFriendlyName()));

		FARFilter Filter;
		Filter.bRecursivePaths = true;
		Filter.PackagePaths.Add(Plugin.ContentRoot);

		// Query for a list of assets in the selected paths
		TArray<FAssetData> AssetsInPlugin;
		AssetRegistry->GetAssets(Filter, AssetsInPlugin);

		ParallelFor(AssetsInPlugin.Num(), [&](int32 AssetIndex)
		{
			const FAssetData& AssetInPlugin = AssetsInPlugin[AssetIndex];
			const FAssetIdentifier AssetId = FAssetIdentifier(AssetInPlugin.PackageName);

			//TODO How can i tell if Asset is an editor only asset and can thus do whatever?  Since it wont matter if it
			// references a bad tag, in the grand scheme.
			const UE::AssetRegistry::FDependencyQuery DependencyRequirements(UE::AssetRegistry::EDependencyQuery::Game);

			TArray<FAssetIdentifier> FoundDependencies;
			if (AssetRegistry->GetDependencies(FAssetIdentifier(AssetInPlugin.PackageName), FoundDependencies, UE::AssetRegistry::EDependencyCategory::All, DependencyRequirements))
			{
				for (const FAssetIdentifier& DependencyId : FoundDependencies)
				{
					// Ensure for all included plugins that they correctly depend on any tag they reference.
					if (DependencyId.ObjectName == NAME_GameplayTag && DependencyId.PackageName == GameplayTagStructPackage)
					{
						TArray<TSharedPtr<IPlugin>> PossibleSources;
						const EDoesPluginDependOnGameplayTagSource DependencyResult = DoesPluginDependOnGameplayTagSource(Manager, Plugin.Plugin, DependencyId.ValueName, PossibleSources);
						if (DependencyResult != EDoesPluginDependOnGameplayTagSource::Yes)
						{
							FString PluginPath = Plugin.ContentRoot.ToString();
							TSharedPtr<IPlugin> ReferencerPlugin = Plugin.Plugin;
							FAssetIdentifier Referencer = AssetId;
							FAssetIdentifier Asset = DependencyId;
							//Violation.AssetPossibleSources = PossibleSources;

							TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
							Message->AddToken(FTextToken::Create(LOCTEXT("GameplayTagReference", "Gameplay Tag Reference")));
							Message->AddToken(FTextToken::Create(FText::FromString(TEXT(":"))));
							Message->AddToken(FTextToken::Create(LOCTEXT("ThePlugin", "The Plugin ")));

							Message->AddToken(
								FAssetNameToken::Create(ReferencerPlugin->GetName())->OnMessageTokenActivated(FOnMessageTokenActivated::CreateLambda([ReferencerPlugin](const TSharedRef<class IMessageToken>&) {
									IPluginsEditorFeature& PluginEditor = IModularFeatures::Get().GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor);
									PluginEditor.OpenPluginEditor(ReferencerPlugin.ToSharedRef(), nullptr, FSimpleDelegate());
									}))
							);

							Message->AddToken(FTextToken::Create(LOCTEXT("ThePluginContains", " contains ")));

							Message->AddToken(
								FAssetNameToken::Create(Referencer.ToString())->OnMessageTokenActivated(FOnMessageTokenActivated::CreateLambda([Referencer](const TSharedRef<class IMessageToken>&) {
									IAssetManagerEditorModule::Get().OpenReferenceViewerUI({ Referencer });
								}))
							);
							Message->AddToken(FTextToken::Create(LOCTEXT("AndItDependsOnGameplayTag", "and it depends on the GameplayTag ")));
							Message->AddToken(
								FAssetNameToken::Create(Asset.ToString())->OnMessageTokenActivated(FOnMessageTokenActivated::CreateLambda([Asset](const TSharedRef<class IMessageToken>&) {
									IAssetManagerEditorModule::Get().OpenReferenceViewerUI({ Asset });
								}))
							);
							if (DependencyResult == EDoesPluginDependOnGameplayTagSource::UnknownTag)
							{
								Message->AddToken(FTextToken::Create(LOCTEXT("FromAndUnknownPlugin", " from a plug-in.  The gameplay tag's source is unknown so it's probably in a plugin that's not registered as a dependency of this plug-in, and nothing else loads the plug-in so we can't figure out where it's supposed to come from.")));
							}
							else
							{
								Message->AddToken(FTextToken::Create(
									FText::FormatNamed(LOCTEXT("GameplayTagFromAssetPlugin", " from {AssetPlugin}. The {ReferencerPlugin} needs to depend on {AssetPlugin} in its .uplugin file."),
										TEXT("ReferencerPlugin"), FText::FromString(ReferencerPlugin->GetName()),
										TEXT("AssetPlugin"), FText::FromString(FPackageName::GetPackageMountPoint(Asset.PackageName.ToString()).ToString())
									)
								));
							}

							{
								FScopeLock Lock(&CS);
								Violations.Add(MoveTemp(Message));
							}
						}
					}
				}
			}
		});
	}

	if (bCancelled)
	{
		return Violations;
	}

	// Excluded plugins
	for (const FGameFeaturePlugin& Plugin : ExcludedPlugins)
	{
		if (SlowTask.ShouldCancel())
		{
			bCancelled = true;
			break;
		}

		SlowTask.EnterProgressFrame(1.f, FText::FromString(Plugin.Plugin->GetFriendlyName()));
		
		FARFilter Filter;
		Filter.bRecursivePaths = true;
		Filter.PackagePaths.Add(Plugin.ContentRoot);

		// Query for a list of assets in the selected paths
		TArray<FAssetData> AssetsInPlugin;
		AssetRegistry->GetAssets(Filter, AssetsInPlugin);
		
		TArray<FAssetIdentifier> AssetIdsInPlugin;
		Algo::Transform(AssetsInPlugin, AssetIdsInPlugin, [](const FAssetData& Asset){ return FAssetIdentifier(Asset.PackageName); });

		// Add all the script packages for the plugin so we can find any references to native classes, structs or enums.
		for (const FString& ScriptPackageName : Plugin.ScriptPackages)
		{
			AssetIdsInPlugin.Add(FAssetIdentifier(*ScriptPackageName));
		}

		// Determine any gameplay tags that are unique to this plugin that might be referenced.
		{
			TArray<FGameplayTag> ContentTags;
			Manager.FindTagsWithSource(Plugin.ContentRoot.ToString(), ContentTags);

			for (const FGameplayTag& Tag : ContentTags)
			{
				if (IsTagOnlyAvailableFromExcludedSources(Manager, Tag, ExcludedPlugins))
				{
					FAssetIdentifier TagId = FAssetIdentifier(FGameplayTag::StaticStruct(), Tag.GetTagName());
					AssetIdsInPlugin.Add(TagId);
				}
			}
		}

		for (const FString& ModuleName : Plugin.ModuleNames)
		{
			TArray<FGameplayTag> NativeTags;
			Manager.FindTagsWithSource(ModuleName, NativeTags);

			for (const FGameplayTag& Tag : NativeTags)
			{
				if (IsTagOnlyAvailableFromExcludedSources(Manager, Tag, ExcludedPlugins))
				{
					FAssetIdentifier TagId = FAssetIdentifier(FGameplayTag::StaticStruct(), Tag.GetTagName());
					AssetIdsInPlugin.Add(TagId);
				}
			}
		}
		
		
		// Burn through the assets in the disabled plugin including any native code packages, we need to figure out
		// from the asset registry who - if anyone references these things.
		ParallelFor(AssetIdsInPlugin.Num(), [&](int32 AssetIndex)
		{
			const FAssetIdentifier& AssetId = AssetIdsInPlugin[AssetIndex];

			TArray<FAssetDependency> Referencers;
			AssetRegistry->GetReferencers(AssetId, Referencers);

			for (const FAssetDependency& Reference : Referencers)
			{
				// If this reference is editor only, we can ignore it.  We only care about game referencing disabled stuff.
				if (!EnumHasAnyFlags(Reference.Properties, UE::AssetRegistry::EDependencyProperty::Game))
				{
					continue;
				}

				const FName PackageMountPoint = FPackageName::GetPackageMountPoint(Reference.AssetId.PackageName.ToString(), false);
				if (IncludedPlugins.ContainsByPredicate([&PackageMountPoint](const FGameFeaturePlugin& Plugin){ return Plugin.ContentRoot == PackageMountPoint; } ))
				{				
					FString PluginPath = Plugin.ContentRoot.ToString();
					FAssetIdentifier Asset = AssetId;
					TSharedPtr<IPlugin> AssetPlugin = Plugin.Plugin;
					FAssetIdentifier Referencer = Reference.AssetId;
					TSharedPtr<IPlugin> ReferencerPlugin = IPluginManager::Get().FindPluginFromPath(Reference.AssetId.PackageName.ToString());

					const FText ReasonFormat = LOCTEXT("IncludedPluginDependsOnExcludedPluginContent", "The {ReferencerPlugin} contains {Referencer} and it depends on {Asset} from {AssetPlugin}. The {AssetPlugin} is disabled or sunset and so can not be referenced.");
	
					TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
					Message->AddToken(FTextToken::Create(LOCTEXT("IncludedExcludedContent", "Disabled Content Reference")));
					Message->AddToken(FTextToken::Create(FText::FromString(TEXT(":"))));
					Message->AddToken(FTextToken::Create(
						FText::FormatNamed(LOCTEXT("TheReferencePlugin","The {ReferencerPlugin} contains "),
							TEXT("ReferencerPlugin"), FText::FromString(ReferencerPlugin->GetName()),
							TEXT("AssetPlugin"), FText::FromString(FPackageName::GetPackageMountPoint(Asset.PackageName.ToString()).ToString())
						)
					));
					Message->AddToken(
						FAssetNameToken::Create(Referencer.ToString())->OnMessageTokenActivated(FOnMessageTokenActivated::CreateLambda([Referencer](const TSharedRef<class IMessageToken>&) {
							IAssetManagerEditorModule::Get().OpenReferenceViewerUI({ Referencer });
						}))
					);
					Message->AddToken(FTextToken::Create(LOCTEXT("AndItDependsOnAsset", " and it depends on ")));
					Message->AddToken(
						FAssetNameToken::Create(Asset.ToString())->OnMessageTokenActivated(FOnMessageTokenActivated::CreateLambda([Asset](const TSharedRef<class IMessageToken>&) {
							IAssetManagerEditorModule::Get().OpenReferenceViewerUI({ Asset });
						}))
					);
					Message->AddToken(FTextToken::Create(
						FText::FormatNamed(LOCTEXT("AssetFromAssetPlugin", " from {AssetPlugin}. The {AssetPlugin} is disabled or sunset and so can not be referenced."),
							TEXT("AssetPlugin"), FText::FromString(FPackageName::GetPackageMountPoint(Asset.PackageName.ToString()).ToString())
						)
					));

					{
						FScopeLock Lock(&CS);
						Violations.Add(MoveTemp(Message));
					}
				}
			}
		});
	}

	return Violations;
}

/*static*/ TArray<TSharedPtr<IPlugin>> SPluginAuditBrowser::GetTagSourcePlugins(const UGameplayTagsManager& Manager, FName TagName)
{
	TArray<TSharedRef<IPlugin>> AllPlugins = IPluginManager::Get().GetDiscoveredPlugins();
	
	TArray<TSharedPtr<IPlugin>> SourcePlugins;
	if (const FGameplayTagSource* TagSource = Manager.FindTagSource(TagName))
	{
		FName TagPackageName;
		switch (TagSource->SourceType)
		{
		case EGameplayTagSourceType::TagList:
			if (TagSource->SourceTagList)
			{
				const FString ContentFilePath = FPaths::GetPath(TagSource->SourceTagList->ConfigFileName) / TEXT("../../Content/");
				FString RootContentPath;
				if (FPackageName::TryConvertFilenameToLongPackageName(ContentFilePath, RootContentPath))
				{
					TagPackageName = FName(*RootContentPath);
				}
			}
			break;
		case EGameplayTagSourceType::DataTable:
			TagPackageName = TagSource->SourceName;
			break;
		case EGameplayTagSourceType::Native:
			TagPackageName = TagSource->SourceName;
		default:
			break;
		}

		if (!TagPackageName.IsNone())
		{
			FString TagPackageNameString = TagPackageName.ToString();
		for (const TSharedRef<IPlugin>& Plugin : AllPlugins)
		{
			FString ContentRoot = FString::Printf(TEXT("/%s/"), *FPaths::GetBaseFilename(Plugin->GetDescriptorFileName()));
				if (TagPackageNameString.StartsWith(ContentRoot))
			{
				SourcePlugins.Add(Plugin);
				continue;
			}
			
			for (const FModuleDescriptor& Module : Plugin->GetDescriptor().Modules)
			{
				if (TagPackageName == Module.Name)
				{
					SourcePlugins.Add(Plugin);
					break;
				}

				const FName ModuleScriptPackage = FPackageName::GetModuleScriptPackageName(Module.Name);
				if (TagPackageName == ModuleScriptPackage)
				{
					SourcePlugins.Add(Plugin);
					break;
				}
			}
		}
	}
	}

	return SourcePlugins;
}

/*static*/ SPluginAuditBrowser::EDoesPluginDependOnGameplayTagSource SPluginAuditBrowser::DoesPluginDependOnGameplayTagSource(const UGameplayTagsManager& Manager, const TSharedPtr<IPlugin>& DependentPlugin, FName TagName, TArray<TSharedPtr<IPlugin>>& OutPossibleSources)
{
	FString Comment;
	TArray<FName> TagSources;
	bool bIsTagExplicit, bIsRestrictedTag, bAllowNonRestrictedChildren;
	if (Manager.GetTagEditorData(TagName, Comment, TagSources, bIsTagExplicit, bIsRestrictedTag, bAllowNonRestrictedChildren))
	{
		const TArray<FModuleDescriptor>& Modules = DependentPlugin->GetDescriptor().Modules;
		const TArray<FPluginReferenceDescriptor>& Plugins = DependentPlugin->GetDescriptor().Plugins;

		TArray<TSharedPtr<IPlugin>> TagSourcePlugins = GetTagSourcePlugins(Manager, TagName);
		if (TagSourcePlugins.Num() == 0)
		{
			// Must be a builtin module?
			return EDoesPluginDependOnGameplayTagSource::Yes;
		}

		for (const TSharedPtr<IPlugin>& TagSourcePlugin : TagSourcePlugins)
		{
			if (TagSourcePlugin == DependentPlugin)
			{
				// The dependent plugin is the source of the tag, so we good.
				return EDoesPluginDependOnGameplayTagSource::Yes;
			}
			else if (Plugins.ContainsByPredicate([&TagSourcePlugin](const FPluginReferenceDescriptor& PluginDescriptor){ return PluginDescriptor.Name == TagSourcePlugin->GetName(); }))
			{
				return EDoesPluginDependOnGameplayTagSource::Yes;
			}
		}

		OutPossibleSources = MoveTemp(TagSourcePlugins);
		return EDoesPluginDependOnGameplayTagSource::No;
	}

	// If there's no source data for the tag then the tag is unknown and hasn't been registered yet.
	return EDoesPluginDependOnGameplayTagSource::UnknownTag;
}

/*static*/ bool SPluginAuditBrowser::IsTagOnlyAvailableFromExcludedSources(const UGameplayTagsManager& Manager, const FGameplayTag& Tag, const TArray<FGameFeaturePlugin>& ExcludedPlugins)
{
	FString Comment;
	TArray<FName> TagSources;
	bool bIsTagExplicit, bIsRestrictedTag, bAllowNonRestrictedChildren;
	if (Manager.GetTagEditorData(Tag.GetTagName(), Comment, TagSources, bIsTagExplicit, bIsRestrictedTag, bAllowNonRestrictedChildren))
	{
		int ExcludedSourcesFound = 0;
		
		for (const FName& TagSourceName : TagSources)
		{
			const FString& TagSourceString = TagSourceName.ToString();

			FName TagSourceIniPackage;
			if (TagSourceString.EndsWith(TEXT(".ini")))
			{
				if (const FGameplayTagSource* TagSource = Manager.FindTagSource(TagSourceName))
				{
					if (ensure(TagSource->SourceTagList))
					{
						FString ContentFilePath = FPaths::GetPath(TagSource->SourceTagList->ConfigFileName) / TEXT("../../Content/");
						FString RootContentPath;
						if (FPackageName::TryConvertFilenameToLongPackageName(ContentFilePath, RootContentPath))
						{
							TagSourceIniPackage = FName(*RootContentPath);
						}
					}
				}
			}
			
			for (const FGameFeaturePlugin& ExcludedPlugin : ExcludedPlugins)
			{
				if (ExcludedPlugin.ModuleNames.Contains(TagSourceString))
				{
					ExcludedSourcesFound++;
					break;
				}

				if (TagSourceString.StartsWith(ExcludedPlugin.ContentRoot.ToString()))
				{
					ExcludedSourcesFound++;
					break;
				}

				if (ExcludedPlugin.ContentRoot == TagSourceIniPackage)
				{
					ExcludedSourcesFound++;
					break;
				}
			}
		}
		
		if (ExcludedSourcesFound == TagSources.Num())
		{
			return true;
		}
	}

	return false;
}

/*static*/ void SPluginAuditBrowser::GetGameFeaturePlugins(const TArray<TSharedRef<IPlugin>>& Plugins, TArray<FGameFeaturePlugin>& GameFeaturePlugins)
{
	const FString BuiltInGameFeaturePluginsFolder = FPaths::ConvertRelativePathToFull(FPaths::ProjectPluginsDir() + TEXT("GameFeatures/"));
	for (const TSharedRef<IPlugin>& Plugin : Plugins)
	{
		const FString& PluginDescriptorFilename = Plugin->GetDescriptorFileName();
		if (!PluginDescriptorFilename.IsEmpty() && FPaths::ConvertRelativePathToFull(PluginDescriptorFilename).StartsWith(BuiltInGameFeaturePluginsFolder))
		{
			FGameFeaturePlugin GameFeaturePlugin;
			GameFeaturePlugin.Plugin = Plugin;
			for (const FModuleDescriptor& Module : Plugin->GetDescriptor().Modules)
			{
				const FString ModuleName = Module.Name.ToString();
				GameFeaturePlugin.ModuleNames.Add(ModuleName);
				GameFeaturePlugin.ScriptPackages.Add(FPackageName::GetModuleScriptPackageName(ModuleName));
			}
			GameFeaturePlugin.ContentRoot = FName(*FString::Printf(TEXT("/%s/"), *FPaths::GetBaseFilename(PluginDescriptorFilename)));

			GameFeaturePlugins.Add(MoveTemp(GameFeaturePlugin));
		}
	}
}

#undef LOCTEXT_NAMESPACE