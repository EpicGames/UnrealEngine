// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Tasks/Task.h"
#include "AssetRegistry/AssetIdentifier.h"
#include "Misc/TextFilter.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Views/SListView.h"

//////////////////////////////////////////////////////////////////////////
// SPluginAuditBrowser

class IPlugin;
struct FGameFeaturePlugin;
class UGameplayTagsManager;
struct FGameplayTag;
class IMessageLogListing;
class IMessageToken;
class SFilterSearchBox;
class STableViewBase;
class ITableRow;
class FTokenizedMessage;

class SPluginAuditBrowser : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPluginAuditBrowser)
	{}

	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs);

private:
	void CreateLogListing();
	void BuildPluginList();
	void RebuildAndFilterPluginList();
	void RefreshToolBar();

	void OnPluginTextFilterChanged();
	void SearchBox_OnPluginSearchTextChanged(const FText& NewText);
	ECheckBoxState GetGlobalDisabledState() const;
	void OnGlobalDisabledStateChanged(ECheckBoxState State);

	FText GetPluginCountText() const;
	TSharedPtr<SWidget> OnContextMenuOpening();
	void OnOpenPluginProperties();
	void OnOpenPluginReferenceViewer();
	void OpenPluginProperties(const FString& PluginName);
	void OpenPluginReferenceViewer(const TSharedRef<IPlugin>& Plugin);

	class FCookedPlugin
	{
	public:
		FCookedPlugin(const TSharedRef<IPlugin>& InPlugin)
			: Plugin(InPlugin)
		{}
		virtual ~FCookedPlugin() = default;

		TSharedRef<IPlugin> Plugin;
		bool bSimulateDisabled = false;
	};

	void OnListViewDoubleClick(TSharedRef<FCookedPlugin> Item);

	TSharedRef<ITableRow> MakeCookedPluginRow(TSharedRef<FCookedPlugin> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	struct FGameFeaturePlugin
	{
		TSharedPtr<IPlugin> Plugin;
		TArray<FString> ModuleNames;
		TArray<FString> ScriptPackages;
		FName ContentRoot;
	};

	enum class EDoesPluginDependOnGameplayTagSource
	{
		Yes,
		No,
		UnknownTag
	};

	void RefreshViolations();

	static TArray<TSharedRef<FTokenizedMessage>> ScanForViolations(TArray<TSharedRef<IPlugin>> InIncludedGameFeaturePlugins, TArray<TSharedRef<IPlugin>> InExcludedGameFeaturePlugins);
	static TArray<TSharedPtr<IPlugin>> GetTagSourcePlugins(const UGameplayTagsManager& Manager, FName TagName);
	static EDoesPluginDependOnGameplayTagSource DoesPluginDependOnGameplayTagSource(const UGameplayTagsManager& Manager, const TSharedPtr<IPlugin>& DependentPlugin, FName TagName, TArray<TSharedPtr<IPlugin>>& OutPossibleSources);
	static bool IsTagOnlyAvailableFromExcludedSources(const UGameplayTagsManager& Manager, const FGameplayTag& Tag, const TArray<FGameFeaturePlugin>& ExcludedPlugins);
	static void GetGameFeaturePlugins(const TArray<TSharedRef<IPlugin>>& Plugins, TArray<FGameFeaturePlugin>& GameFeaturePlugins);

private:
	typedef TTextFilter<const IPlugin*> FCookedPluginTextFilter;

	TArray<TSharedRef<IPlugin>> IncludedGameFeaturePlugins;
	TArray<TSharedRef<IPlugin>> ExcludedGameFeaturePlugins;

	TArray<TSharedRef<FCookedPlugin>> CookedPlugins;
	TArray<TSharedRef<FCookedPlugin>> FilteredCookedPlugins;

	TSharedPtr<IMessageLogListing> LogListing;

	/** The list view widget for our plugins list */
	TSharedPtr<SListView<TSharedRef<FCookedPlugin>>> PluginListView;

	/** The plugin search box widget */
	TSharedPtr<class SSearchBox> SearchBoxPtr;

	/** Text filter object for typing in filter text to the search box */
	TSharedPtr<FCookedPluginTextFilter> PluginTextFilter;

	bool bGlobalDisabledState = false;
};